// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogMacros.h"
#include "RequiredProgramMainCPPInclude.h"

#include "UnrealMultiUserServerRun.h"
#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY_STATIC(LogMultiUserServer, Log, All)

IMPLEMENT_APPLICATION(UnrealMultiUserServer, "UnrealMultiUserServer");

#if PLATFORM_MAC // On Mac, to get a properly logging console that play nice, we need to build a mac application (.app) rather than a console application.

#include "Mac/MacSystemIncludes.h"
#include "Apple/ScopeAutoreleasePool.h"

class CommandLineArguments
{
public:
	CommandLineArguments() : ArgC(0), ArgV(nullptr) {}
	CommandLineArguments(int InArgC, char* InUtf8ArgV[]) { Init(InArgC, InUtf8ArgV); }

	void Init(int InArgC, char* InUtf8ArgV[])
	{
		ArgC = InArgC;
		ArgV = new TCHAR*[ArgC];
		for (int32 a = 0; a < ArgC; a++)
		{
			FUTF8ToTCHAR ConvertFromUtf8(InUtf8ArgV[a]);
			ArgV[a] = new TCHAR[ConvertFromUtf8.Length() + 1];
			FCString::Strncpy(ArgV[a], ConvertFromUtf8.Get(), ConvertFromUtf8.Length() + 1);
		}
	}

	~CommandLineArguments()
	{
		for (int32 a = 0; a < ArgC; a++)
		{
			delete[] ArgV[a];
		}
		delete[] ArgV;
	}

	int ArgC;
	TCHAR** ArgV;
};

#import <Foundation/NSAppleEventDescriptor.h>
#import <Carbon/Carbon.h>
#include "Mac/CocoaThread.h"

static CommandLineArguments GSavedCommandLine;

@interface UEAppDelegate : NSObject <NSApplicationDelegate>
@end

@implementation UEAppDelegate

//handler for the quit apple event used by the Dock menu
- (void)handleQuitEvent:(NSAppleEventDescriptor*)Event withReplyEvent:(NSAppleEventDescriptor*)ReplyEvent
{
	[NSApp terminate:self];
}

- (void)runGameThread:(id)Arg
{
	FPlatformMisc::SetGracefulTerminationHandler();
	FPlatformMisc::SetCrashHandler(nullptr);
	RunUnrealMultiUserServer(GSavedCommandLine.ArgC, GSavedCommandLine.ArgV);
}

- (BOOL)applicationShouldTerminateAfterLastWindowClosed:(NSApplication *)Sender
{
	return true;
}

- (NSApplicationTerminateReply)applicationShouldTerminate:(NSApplication *)Sender
{
	if(!IsEngineExitRequested() || ([NSThread gameThread] && [NSThread gameThread] != [NSThread mainThread]))
	{
		FString Reason;
		NSAppleEventDescriptor* AppleEventDesc = [[NSAppleEventManager sharedAppleEventManager] currentAppleEvent];
		NSAppleEventDescriptor* WhyDesc = [AppleEventDesc attributeDescriptorForKeyword:kEventParamReason];
		OSType Why = [WhyDesc typeCodeValue];
		if (Why == kAEShutDown)
		{
			Reason = TEXT("System Shutting Down");
		}
		else if (Why == kAERestart)
		{
			Reason = TEXT("System Restarting");
		}
		else if (Why == kAEReallyLogOut)
		{
			Reason = TEXT("User Logging Out");
		}
		else
		{
			Reason = TEXT("User Quitting (CMD-Q/Quit Menu)");
		}

		UE_LOGF(LogMultiUserServer, Warning, "*** INTERRUPTED *** : SHUTTING DOWN");
		UE_LOGF(LogMultiUserServer, Warning, "*** INTERRUPTED *** : %ls", *Reason);
		RequestEngineExit(*FString::Printf(TEXT("UnrealMultiUserServer Requesting Exit: %s"), *Reason));
		
		return NSTerminateLater;
	}
	else
	{
		return NSTerminateNow;
	}
}

- (void)applicationDidFinishLaunching:(NSNotification *)Notification
{
	//install the custom quit event handler
	NSAppleEventManager* appleEventManager = [NSAppleEventManager sharedAppleEventManager];
	[appleEventManager setEventHandler:self andSelector:@selector(handleQuitEvent:withReplyEvent:) forEventClass:kCoreEventClass andEventID:kAEQuitApplication];

	// Add a menu bar to the application.
	id menubar = [[NSMenu new] autorelease];
	id appMenuItem = [[NSMenuItem new] autorelease];
	[menubar addItem:appMenuItem];
	[NSApp setMainMenu:menubar];

	// Populate the menu bar.
	id appMenu = [[NSMenu new] autorelease];
	id quitMenuItem = [[[NSMenuItem alloc] initWithTitle:NSLOCTEXT("UMUS_Quit", "QuitApp", "Quit").ToString().GetNSString() action:@selector(terminate:) keyEquivalent:@"q"] autorelease];
	[appMenu addItem:quitMenuItem];
	[appMenuItem setSubmenu:appMenu];

	RunGameThread(self, @selector(runGameThread:));
}

@end

int main(int argc, char *argv[])
{
	// Record the command line.
	GSavedCommandLine.Init(argc, argv);

	// Launch the application.
	SCOPED_AUTORELEASE_POOL;
	[NSApplication sharedApplication];
	[NSApp setDelegate:[UEAppDelegate new]];
	[NSApp run];
	return 0;
}


#else // Windows/Linux

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	return RunUnrealMultiUserServer(ArgC, ArgV);
}

#endif
