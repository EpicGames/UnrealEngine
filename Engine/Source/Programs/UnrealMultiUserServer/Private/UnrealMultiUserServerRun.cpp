// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealMultiUserServerRun.h"

#include "ConcertSettings.h"
#include "ConcertSyncServerLoop.h"

#include "Misc/CommandLine.h"

int32 RunUnrealMultiUserServer(int ArgC, TCHAR* ArgV[])
{
	FString Role(TEXT("MultiUser"));
	FConcertSyncServerLoopInitArgs ServerLoopInitArgs;
	ServerLoopInitArgs.SessionFlags = EConcertSyncSessionFlags::Default_MultiUserSession;
	ServerLoopInitArgs.ServiceRole = Role;
	ServerLoopInitArgs.ServiceFriendlyName = TEXT("Multi-User Editing Server");
	ServerLoopInitArgs.GetFileSharingServiceFunc = [Role]() -> TSharedPtr<IConcertFileSharingService>
	{
		return IConcertSyncServerModule::Get().CreateFileSharingService(Role);
	};
	
	ServerLoopInitArgs.GetServerConfigFunc = [Role]() -> const UConcertServerConfig*
	{
		UConcertServerConfig* ServerConfig = IConcertSyncServerModule::Get().ParseServerSettings(FCommandLine::Get());
		if (ServerConfig->WorkingDir.IsEmpty())
		{
			ServerConfig->WorkingDir = FPaths::ProjectIntermediateDir() / Role;
		}
		if (ServerConfig->ArchiveDir.IsEmpty())
		{
			ServerConfig->ArchiveDir = FPaths::ProjectSavedDir() / Role;
		}
		return ServerConfig;
	};

	return ConcertSyncServerLoop(*FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr), ServerLoopInitArgs);
}
