// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

int32 RunUnrealMultiUserServer(int ArgC, TCHAR* ArgV[]);
