// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealMultiUserServer : ModuleRules
{
	public UnrealMultiUserServer(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Concert",
				"ConcertServer",
				"TraceInsights",
				"ApplicationCore",					// for LaunchEngineLoop.cpp dependency
				"Projects",							// for LaunchEngineLoop.cpp dependency
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
				"ConcertSyncCore",
				"ConcertSyncServer"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"ConcertSyncServer"
			}
		);
	}
}
