// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealMultiUserServerTarget : TargetRules
{
	public UnrealMultiUserServerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealMultiUserServer";
		AdditionalPlugins.Add("UdpMessaging");
		AdditionalPlugins.Add("QuicMessaging");
		AdditionalPlugins.Add("ConcertSyncServer");

		// This app compiles against Core/CoreUObject, but not the Engine or Editor, so compile out Engine and Editor references from Core/CoreUObject
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bBuildWithEditorOnlyData = false;

		// Enable Developer plugins (like Concert!)
		bCompileWithPluginSupport = true;
		bBuildDeveloperTools = true;

		// The Multi-User server is meant to be a console application (no window), but on MacOS, to get a proper log console, a full application must be built.
		bIsBuildingConsoleApplication = Target.Platform != UnrealTargetPlatform.Mac;

		GlobalDefinitions.Add("UE_LOG_CONCERT_DEBUG_VERBOSITY_LEVEL=Log");
		bEnableTrace = true;
	}
}
